// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class PinnedCommandList : ModuleRules
{
    public PinnedCommandList(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject",
				"ApplicationCore",
                "Slate",
                "SlateCore",
                "InputCore",
            }
        );
    }
}
