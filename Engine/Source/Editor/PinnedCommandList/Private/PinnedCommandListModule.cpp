// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPinnedCommandListModule.h"
#include "Modules/ModuleManager.h"
#include "SPinnedCommandList.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class IPinnedCommandList;

class FPinnedCommandListModule : public IPinnedCommandListModule
{
public:
	virtual TSharedRef<IPinnedCommandList> CreatePinnedCommandList(const FName& InContextName) override
	{
		return SNew(SPinnedCommandList, InContextName);
	}
};

IMPLEMENT_MODULE(FPinnedCommandListModule, PinnedCommandList)
