// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PinnedCommandListSettings.generated.h"

UENUM()
enum class EPinnedCommandListType : uint8
{
	/** A simple command */
	Command,

	/** A registered custom widget */
	CustomWidget,
};

/** A command and its context */
USTRUCT()
struct FPinnedCommandListCommand
{
	GENERATED_BODY()

	/** The name of the command */
	UPROPERTY(config)
	FName Name;

	/** The name of the command binding if this is a command */
	UPROPERTY(config)
	FName Binding;

	/** What type of command this is */
	UPROPERTY(config)
	EPinnedCommandListType Type = EPinnedCommandListType::Command;
};

/** A pinned command list context, allowing us to persist a set of pinned commands */
USTRUCT()
struct FPinnedCommandListContext
{
	GENERATED_BODY()

	/** The name of the context that this command list is displayed within */
	UPROPERTY(config)
	FName Name;

	/** The commands to display */
	UPROPERTY(config)
	TArray<FPinnedCommandListCommand> Commands;
};

/** A persistent set of pinned commands */
UCLASS(config=EditorPerProjectUserSettings)
class UPinnedCommandListSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Persistent pinned command contexts */
	UPROPERTY(config)
	TArray<FPinnedCommandListContext> Contexts;
};