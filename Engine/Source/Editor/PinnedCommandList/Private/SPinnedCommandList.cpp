// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPinnedCommandList.h"

#include "CoreTypes.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/PlatformCrt.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Visibility.h"
#include "Layout/WidgetPath.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "PinnedCommandListSettings.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UICommandList_Pinnable.h"
#include "UObject/UObjectBase.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
struct FGeometry;

#define LOCTEXT_NAMESPACE "PinnedCommandList"

/**
 * A single command in the command list.
 */
class SCommand : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam( FOnRequestRemove, const TSharedRef<SCommand>& /*CommandToRemove*/ );
	DECLARE_DELEGATE( FOnRequestRemoveAll );

	SLATE_BEGIN_ARGS( SCommand )
		: _StyleSet(&FAppStyle::Get())
		, _StyleName(TEXT("SkeletonTree.PinnedCommandList"))
		, _CustomWidgetPadding(2.0f, 1.0f)
	{}

		/** Invoked when a request to remove this command originated from within this command */
		SLATE_EVENT( FOnRequestRemove, OnRequestRemove )

		/** Invoked when a request to remove all commands originated from within this command */
		SLATE_EVENT( FOnRequestRemoveAll, OnRequestRemoveAll )

		/** Invoked when a request to remove all but the specified command originated from within this command */
		SLATE_EVENT( FOnRequestRemove, OnRequestRemoveAllButThis )

		/** The slate style to use when constructing command widgets */
		SLATE_ARGUMENT( const ISlateStyle*, StyleSet )

		/** The menu style name to use when constructing command widgets */
		SLATE_ARGUMENT( FName, StyleName )

		/** Command info if we are using a basic command */
		SLATE_ARGUMENT(TSharedPtr<const FUICommandInfo>, CommandInfo)

		/** Command list if we are using a basic command */
		SLATE_ARGUMENT(TSharedPtr<const FUICommandList>, CommandList)

		/** Pinnable command list if one was supplied */
		SLATE_ARGUMENT(TSharedPtr<const FUICommandList_Pinnable>, CommandListPinnable)

		/** Custom widget */
		SLATE_ARGUMENT(TSharedPtr<SWidget>, CustomWidget)

		/** Custom widget display name */
		SLATE_ATTRIBUTE(FText, CustomWidgetDisplayName)

		/** Identifier used for custom widget */
		SLATE_ARGUMENT(FName, CustomWidgetIdentifier)

		/** Identifier used for custom widget */
		SLATE_ARGUMENT(FMargin, CustomWidgetPadding)

		/** Whether to display the custom widget label */
		SLATE_ARGUMENT(bool, ShowCustomWidgetLabel)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs)
	{
		CommandInfo = InArgs._CommandInfo;
		CommandList = InArgs._CommandList;
		CommandListPinnable = InArgs._CommandListPinnable;
		CustomWidget = InArgs._CustomWidget;
		CustomWidgetDisplayName = InArgs._CustomWidgetDisplayName;
		CustomWidgetIdentifier = InArgs._CustomWidgetIdentifier;
		OnRequestRemove = InArgs._OnRequestRemove;
		OnRequestRemoveAll = InArgs._OnRequestRemoveAll;
		OnRequestRemoveAllButThis = InArgs._OnRequestRemoveAllButThis;

		// Using a menu builder here is slightly wasteful, but as we cant construct
		// a menu item individually it will have to do for now.
		const bool bInShouldCloseWindowAfterMenuSelection = false;
		FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, CommandList.Pin(), nullptr, false, InArgs._StyleSet, false);
		MenuBuilder.SetStyle(InArgs._StyleSet, InArgs._StyleName);

		if(CommandInfo.IsValid() && CommandList.IsValid())
		{
			MenuBuilder.AddMenuEntry(CommandInfo.Pin());
		}
		else if(CustomWidget.IsValid())
		{
			TSharedRef<SWidget> CustomWidgetContainer = 
				SNew(SBorder)
				.Padding(0)
				.BorderImage(InArgs._StyleSet->GetBrush(InArgs._StyleName, ".Background"))
				.ForegroundColor(FCoreStyle::Get().GetSlateColor("DefaultForeground"))
				[
					SNew(SButton)
					.ButtonStyle(InArgs._StyleSet, ISlateStyle::Join(InArgs._StyleName, ".Button"))
					.ContentPadding(InArgs._CustomWidgetPadding)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Visibility(InArgs._ShowCustomWidgetLabel ? EVisibility::Visible : EVisibility::Collapsed)
							.TextStyle(InArgs._StyleSet, ISlateStyle::Join(InArgs._StyleName, ".Label"))
							.Text(CustomWidgetDisplayName)
						]
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							CustomWidget.ToSharedRef()
						]
					]
				];

			MenuBuilder.AddWidget(CustomWidgetContainer, FText(), true, false);
		}
		else
		{
			check(false);	// We must have either a valid command or a custom widget
		}
		
		ChildSlot
		[
			MenuBuilder.MakeWidget()
		];
	}

	TWeakPtr<const FUICommandInfo> GetCommandInfo() const
	{
		return CommandInfo;
	}

	TWeakPtr<const FUICommandList> GetCommandList() const
	{
		return CommandList;
	}

	TWeakPtr<const FUICommandList_Pinnable> GetPinnableCommandList() const
	{
		return CommandListPinnable;
	}

	void SetPinnableCommandList(const TSharedRef<const FUICommandList_Pinnable>& InUICommandList)
	{
		CommandListPinnable = InUICommandList;
	}

	FName GetCommandIdentifier() const
	{
		return CommandInfo.IsValid() ? CommandInfo.Pin()->GetCommandName() : CustomWidgetIdentifier;
	}

	bool IsCustomWidget() const
	{
		return CustomWidget.IsValid();
	}

private:

	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			const bool bInShouldCloseWindowAfterMenuSelection = true;
			FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr);

			MenuBuilder.BeginSection("CommandOptions", LOCTEXT("CommandOptionsHeading", "Command Options"));
			{
				MenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("RemoveCommand", "Remove: {0}"), GetCommandName()),
					LOCTEXT("RemoveCommandTooltip", "Remove this command from the list (Shift-click)"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SCommand::RemoveCommand))
					);

				MenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("RemoveAllButThis", "Remove All But: {0}"), GetCommandName()),
					LOCTEXT("RemoveAllButThisTooltip", "Removes all commands apart from this one from the list."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SCommand::RemoveAllCommandsButThis))
					);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("RemoveAllCommands", "Remove All Commands"),
					LOCTEXT("RemoveAllCommandsTooltip", "Removes all commands from the list."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SCommand::RemoveAllCommands))
					);
			}
			MenuBuilder.EndSection();

			FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
			FSlateApplication::Get().PushMenu(
				AsShared(),
				WidgetPath,
				MenuBuilder.MakeWidget(),
				MouseEvent.GetScreenSpacePosition(),
				FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
				);

			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		if (HasMouseCapture() && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			// If shift is held we probably removed another widget in OnPreviewMouseButtonDown, so ignore here
			RemoveCommand();
			return FReply::Handled().ReleaseMouseCapture();
		}

		return FReply::Unhandled();
	}

	virtual FReply OnPreviewMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		// Shift-LMB removes the item
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && MouseEvent.IsShiftDown())
		{
			return FReply::Handled().CaptureMouse(SharedThis(this));
		}

		return FReply::Unhandled();
	}

	/** Removes this command from the command list */
	void RemoveCommand()
	{
		OnRequestRemove.ExecuteIfBound(SharedThis(this));
	}

	/** Removes all commands in the list */
	void RemoveAllCommands()
	{
		OnRequestRemoveAll.ExecuteIfBound();
	}

	/** Removes all but the specified command from the list */
	void RemoveAllCommandsButThis()
	{
		OnRequestRemoveAllButThis.ExecuteIfBound(SharedThis(this));
	}

	/** Returns the display name for this command */
	FText GetCommandName() const
	{
		return CommandInfo.IsValid() ? CommandInfo.Pin()->GetLabel() : CustomWidgetDisplayName.Get();
	}

private:
	/** Invoked when a request to remove this command originated from within this command */
	FOnRequestRemove OnRequestRemove;

	/** Invoked when a request to remove all commands originated from within this command */
	FOnRequestRemoveAll OnRequestRemoveAll;

	/** Invoked when a request to remove all but the specified command originated from within this command */
	FOnRequestRemove OnRequestRemoveAllButThis;

	/** Command info for this command */
	TWeakPtr<const FUICommandInfo> CommandInfo;

	/** Command list context in which to process the command */
	TWeakPtr<const FUICommandList> CommandList;

	/** Pinnable command list context, for extra info if available */
	TWeakPtr<const FUICommandList_Pinnable> CommandListPinnable;

	/** Custom widget */
	TSharedPtr<SWidget> CustomWidget;

	/** Identifier if using a custom widget */
	FName CustomWidgetIdentifier;

	/** Display name if using a custom widget */
	TAttribute<FText> CustomWidgetDisplayName;
};

SPinnedCommandList::SPinnedCommandList()
	: StyleSet(&FAppStyle::Get())
	, StyleName(TEXT("PinnedCommandList"))
{
}

SPinnedCommandList::~SPinnedCommandList()
{
	if(UObjectInitialized())
	{
		SaveSettings();
	}
}

void SPinnedCommandList::Construct( const FArguments& InArgs, const FName& InContextName )
{
	ContextName = InContextName;
	OnGetContextMenu = InArgs._OnGetContextMenu;
	OnCommandsChanged = InArgs._OnCommandsChanged;

	ChildSlot
	[
		SAssignNew(CommandBox, SWrapBox)
		.UseAllottedSize(true)
		.InnerSlotPadding(FVector2D(0.0f, 0.0f))
	];
}

FReply SPinnedCommandList::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
	{
		if ( OnGetContextMenu.IsBound() )
		{
			FReply Reply = FReply::Handled().ReleaseMouseCapture();

			// Get the context menu content. If NULL, don't open a menu.
			TSharedPtr<SWidget> MenuContent = OnGetContextMenu.Execute();

			if ( MenuContent.IsValid() )
			{
				FVector2D SummonLocation = MouseEvent.GetScreenSpacePosition();
				FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
				FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuContent.ToSharedRef(), SummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
			}

			return Reply;
		}
	}

	return FReply::Unhandled();
}

bool SPinnedCommandList::HasAnyCommands() const
{
	return Commands.Num() > 0;
}

void SPinnedCommandList::RemoveAllCommands()
{
	if (HasAnyCommands())
	{
		CommandBox->ClearChildren();
		Commands.Empty();

		// Notify that the displayed commands changed
		OnCommandsChanged.ExecuteIfBound();
	}
}

void SPinnedCommandList::RemoveAllCommandsButThis(const TSharedRef<SCommand>& CommandToRemove)
{
	if (HasAnyCommands())
	{
		for(int32 ChildIndex = Commands.Num() - 1; ChildIndex >= 0; --ChildIndex)
		{
			if(Commands[ChildIndex] != CommandToRemove)
			{
				CommandBox->RemoveSlot(Commands[ChildIndex]);
				Commands.RemoveAt(ChildIndex);
			}
		}

		// Notify that the displayed commands changed
		OnCommandsChanged.ExecuteIfBound();
	}
}

void SPinnedCommandList::SetStyle(const ISlateStyle* InStyleSet, const FName& InStyleName)
{
	StyleSet = InStyleSet;
	StyleName = InStyleName;
}

void SPinnedCommandList::SaveSettings() const
{
	if(ContextName != NAME_None)
	{
		UPinnedCommandListSettings* Settings = GetMutableDefault<UPinnedCommandListSettings>();

		FPinnedCommandListContext* FoundContext = Settings->Contexts.FindByPredicate([this](const FPinnedCommandListContext& Context)
		{
			return Context.Name == ContextName;
		});

		if(FoundContext == nullptr)
		{
			FoundContext = &Settings->Contexts[Settings->Contexts.AddDefaulted()];
			FoundContext->Name = ContextName;
		}

		FoundContext->Commands.Reset();
	
		for (const TSharedRef<SCommand>& Command : Commands)
		{
			TWeakPtr<const FUICommandInfo> CommandInfo = Command->GetCommandInfo();
			if(CommandInfo.IsValid())
			{
				FPinnedCommandListCommand PersistedCommand;
				PersistedCommand.Name = CommandInfo.Pin()->GetCommandName();
				PersistedCommand.Binding = CommandInfo.Pin()->GetBindingContext();
				PersistedCommand.Type = EPinnedCommandListType::Command;
				FoundContext->Commands.Add(PersistedCommand);
			}
			else if(Command->IsCustomWidget())
			{
				FPinnedCommandListCommand PersistedCommand;
				PersistedCommand.Name = Command->GetCommandIdentifier();
				PersistedCommand.Type = EPinnedCommandListType::CustomWidget;
				FoundContext->Commands.Add(PersistedCommand);
			}
		}

		Settings->SaveConfig();
	}
}

void SPinnedCommandList::LoadSettings()
{
	if(ContextName != NAME_None)
	{
		GetMutableDefault<UPinnedCommandListSettings>()->LoadConfig();

		const UPinnedCommandListSettings* Settings = GetDefault<UPinnedCommandListSettings>();

		const FPinnedCommandListContext* FoundContext = Settings->Contexts.FindByPredicate([this](const FPinnedCommandListContext& Context)
		{
			return Context.Name == ContextName;
		});

		if(FoundContext)
		{
			for(const FPinnedCommandListCommand& Command : FoundContext->Commands)
			{
				if(Command.Type == EPinnedCommandListType::Command)
				{
					TSharedPtr<FUICommandInfo> CommandInfo = FInputBindingManager::Get().FindCommandInContext(Command.Binding, Command.Name);
					if(CommandInfo.IsValid())
					{
						// now search our bound command lists and add the action if we find one
						for(TWeakPtr<const FUICommandList>& CommandList : BoundCommandLists)
						{
							if(CommandList.IsValid() && CommandList.Pin()->IsActionMapped(CommandInfo))
							{
								AddCommand_Internal(CommandInfo.ToSharedRef(), CommandList.Pin().ToSharedRef());
							}
						}

						for(TWeakPtr<const FUICommandList_Pinnable>& PinnableCommandList : BoundPinnableCommandLists)
						{
							if(PinnableCommandList.IsValid() && PinnableCommandList.Pin()->IsActionMapped(CommandInfo))
							{
								AddCommand_Internal(CommandInfo.ToSharedRef(), PinnableCommandList.Pin().ToSharedRef(), PinnableCommandList.Pin().ToSharedRef());
							}
						}
					}
				}
				else if(Command.Type == EPinnedCommandListType::CustomWidget)
				{
					AddCustomWidget_Internal(Command.Name);
				}
			}
		}
	}
}

void SPinnedCommandList::BindCommandList(const TSharedRef<const FUICommandList>& InUICommandList)
{
	int32 Index = INDEX_NONE;
	if (!BoundCommandLists.Find(InUICommandList, Index))
	{
		BoundCommandLists.Add(InUICommandList);

		LoadSettings();
	}
}

void SPinnedCommandList::BindCommandList(const TSharedRef<FUICommandList_Pinnable>& InUICommandList_Pinnable)
{
	int32 Index = INDEX_NONE;
	if (!BoundPinnableCommandLists.Find(InUICommandList_Pinnable, Index))
	{
		BoundPinnableCommandLists.Add(InUICommandList_Pinnable);

		LoadSettings();
	}

	InUICommandList_Pinnable->OnExecuteAction().AddSP(this, &SPinnedCommandList::HandleExecuteAction);
	InUICommandList_Pinnable->OnCustomWidgetInteraction().AddSP(this, &SPinnedCommandList::HandleCustomWidgetInteraction);
}

void SPinnedCommandList::RegisterCustomWidget(IPinnedCommandList::FOnGenerateCustomWidget InOnGenerateCustomWidget, FName InCustomWidgetIdentifier, TAttribute<FText> InCustomWidgetDisplayName, FMargin InCustomWidgetPadding, bool bInShowLabel)
{
	// Check if the widget is registered already
	FRegisteredCustomWidget* RegisteredWidget = CustomWidgets.FindByPredicate([InCustomWidgetIdentifier](const FRegisteredCustomWidget& InRegisteredWidget)
	{
		return InCustomWidgetIdentifier == InRegisteredWidget.CustomWidgetIdentifier;
	});

	if (RegisteredWidget == nullptr)
	{
		check(InOnGenerateCustomWidget.IsBound());

		FRegisteredCustomWidget NewCustomWidget;
		NewCustomWidget.CustomWidgetIdentifier = InCustomWidgetIdentifier;
		NewCustomWidget.CustomWidgetDisplayName = InCustomWidgetDisplayName;
		NewCustomWidget.OnGenerateCustomWidget = InOnGenerateCustomWidget;
		NewCustomWidget.CustomWidgetPadding = InCustomWidgetPadding;
		NewCustomWidget.bShowLabel = bInShowLabel;

		CustomWidgets.Add(NewCustomWidget);
		LoadSettings();
	}
}

void SPinnedCommandList::HandleExecuteAction(const TSharedRef<const FUICommandInfo>& InCommandInfo, const TSharedRef<const FUICommandList_Pinnable>& InUICommandList)
{
	if(FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Shift))
	{
		AddCommand_Internal(InCommandInfo, InUICommandList, InUICommandList);
	}
}

void SPinnedCommandList::AddCommand(const TSharedRef<const FUICommandInfo>& InCommandInfo, const TSharedRef<const FUICommandList>& InUICommandList)
{
	AddCommand_Internal(InCommandInfo, InUICommandList);
}

TSharedRef<SCommand> SPinnedCommandList::AddCommand_Internal(const TSharedRef<const FUICommandInfo>& InCommandInfo, const TSharedRef<const FUICommandList>& InUICommandList, const TSharedPtr<const FUICommandList_Pinnable>& InUICommandListPinnable)
{
	// check we dont already have this command
	TSharedRef<SCommand>* ExistingCommand = Commands.FindByPredicate([InCommandInfo](TSharedRef<SCommand>& InCommand)
	{
		return InCommand->GetCommandIdentifier() == InCommandInfo->GetCommandName();
	});

	if(ExistingCommand == nullptr)
	{
		TSharedRef<SCommand> NewCommand =
			SNew(SCommand)
			.OnRequestRemove(this, &SPinnedCommandList::RemoveCommandWidget)
			.OnRequestRemoveAll(this, &SPinnedCommandList::RemoveAllCommands)
			.OnRequestRemoveAllButThis(this, &SPinnedCommandList::RemoveAllCommandsButThis)
			.StyleSet(StyleSet)
			.StyleName(StyleName)
			.CommandInfo(InCommandInfo)
			.CommandList(InUICommandList)
			.CommandListPinnable(InUICommandListPinnable);

		AddCommandWidget(NewCommand);

		return NewCommand;
	}

	return *ExistingCommand;
}

void SPinnedCommandList::HandleCustomWidgetInteraction(FName InCustomWidgetIdentifier, const TSharedRef<const FUICommandList_Pinnable>& InUICommandList)
{
	if(FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Shift))
	{
		AddCustomWidget_Internal(InCustomWidgetIdentifier, InUICommandList);
	}
}

void SPinnedCommandList::AddCustomWidget(FName InCustomWidgetIdentifier)
{
	if(FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Shift))
	{
		AddCustomWidget_Internal(InCustomWidgetIdentifier);
	}
}

TSharedPtr<SCommand> SPinnedCommandList::AddCustomWidget_Internal(FName InCustomWidgetIdentifier, const TSharedPtr<const FUICommandList_Pinnable>& InUICommandListPinnable)
{
	// Check the widget is registered
	FRegisteredCustomWidget* RegisteredWidget = CustomWidgets.FindByPredicate([InCustomWidgetIdentifier](const FRegisteredCustomWidget& InRegisteredWidget)
	{
		return InCustomWidgetIdentifier == InRegisteredWidget.CustomWidgetIdentifier;
	});

	if(RegisteredWidget)
	{
		// check we dont already have this command
		TSharedRef<SCommand>* ExistingCommand = Commands.FindByPredicate([InCustomWidgetIdentifier](TSharedRef<SCommand>& InCommand)
		{
			return InCommand->GetCommandIdentifier() == InCustomWidgetIdentifier;
		});

		if(ExistingCommand == nullptr)
		{
			TSharedRef<SCommand> NewCommand =
				SNew(SCommand)
				.OnRequestRemove(this, &SPinnedCommandList::RemoveCommandWidget)
				.OnRequestRemoveAll(this, &SPinnedCommandList::RemoveAllCommands)
				.OnRequestRemoveAllButThis(this, &SPinnedCommandList::RemoveAllCommandsButThis)
				.StyleSet(StyleSet)
				.StyleName(StyleName)
				.CustomWidget(RegisteredWidget->OnGenerateCustomWidget.Execute())
				.CustomWidgetIdentifier(RegisteredWidget->CustomWidgetIdentifier)
				.CustomWidgetDisplayName(RegisteredWidget->CustomWidgetDisplayName)
				.CustomWidgetPadding(RegisteredWidget->CustomWidgetPadding)
				.CommandListPinnable(InUICommandListPinnable)
				.ShowCustomWidgetLabel(RegisteredWidget->bShowLabel);

			AddCommandWidget(NewCommand);

			return NewCommand;
		}

		return *ExistingCommand;
	}

	return nullptr;
}

void SPinnedCommandList::AddCommandWidget(const TSharedRef<SCommand>& CommandToAdd)
{
	Commands.Add(CommandToAdd);

	// Sort commands - this will re-add the widgets to slots
	SortCommands();
}

void SPinnedCommandList::RemoveCommand(const TSharedRef<const FUICommandInfo>& InCommandInfo)
{
	TSharedPtr<SCommand> ComandToRemove;
	for (TSharedRef<SCommand>& Command : Commands)
	{
		const TWeakPtr<const FUICommandInfo>& CommandInfo = Command->GetCommandInfo();
		if (CommandInfo.IsValid() && CommandInfo.Pin().ToSharedRef() == InCommandInfo)
		{
			ComandToRemove = Command;
			break;
		}
	}

	if (ComandToRemove.IsValid())
	{
		RemoveCommandWidget(ComandToRemove.ToSharedRef());
	}
}

void SPinnedCommandList::RemoveCustomWidget(FName InCustomWidgetIdentifier)
{
	TSharedPtr<SCommand> ComandToRemove;
	for (TSharedRef<SCommand>& Command : Commands)
	{
		if(Command->GetCommandIdentifier() == InCustomWidgetIdentifier)
		{
			ComandToRemove = Command;
			break;
		}
	}

	if (ComandToRemove.IsValid())
	{
		RemoveCommandWidget(ComandToRemove.ToSharedRef());
	}
}

void SPinnedCommandList::RemoveCommandWidget(const TSharedRef<SCommand>& CommandToRemove)
{
	CommandBox->RemoveSlot(CommandToRemove);
	Commands.Remove(CommandToRemove);

	RefreshCommandWidgets();

	// Notify that the commands changed
	OnCommandsChanged.ExecuteIfBound();
}

void SPinnedCommandList::OnResetCommands()
{
	RemoveAllCommands();
}

void SPinnedCommandList::SortCommands()
{
	// re-sort command widgets
	Commands.Sort([](const TSharedRef<SCommand>& InCommand0, const TSharedRef<SCommand>& InCommand1)
	{
		// Sort via index if we are using pinnable command lists for both commands
		FName CommandIdentifier0 = InCommand0->GetCommandIdentifier();
		TSharedPtr<const FUICommandList_Pinnable> CommandList0 = InCommand0->GetPinnableCommandList().Pin();

		FName CommandIdentifier1 = InCommand1->GetCommandIdentifier();
		TSharedPtr<const FUICommandList_Pinnable> CommandList1 = InCommand1->GetPinnableCommandList().Pin();

		if(CommandList0.IsValid() && CommandList1.IsValid())
		{
			const int32 Index0 = CommandList0->GetMappedCommandIndex(CommandIdentifier0);
			const int32 Index1 = CommandList1->GetMappedCommandIndex(CommandIdentifier1);

			return Index0 < Index1;
		}

		// fallback to lexical sort
		return CommandIdentifier0.LexicalLess(CommandIdentifier1);
	});

	RefreshCommandWidgets();
}

void SPinnedCommandList::RefreshCommandWidgets()
{
	// Empty the current slots
	CommandBox->ClearChildren();

	// re-add command widgets to slots
	for (int32 CommandIndex = 0; CommandIndex < Commands.Num(); ++CommandIndex)
	{
		// Calculate padding
		FMargin Padding(0.0f, 2.0f, 4.0f, 2.0f);
		TSharedRef<SCommand>& Command = Commands[CommandIndex];
		TSharedPtr<const FUICommandList_Pinnable> CommandList = Command->GetPinnableCommandList().Pin();
		FName Group = CommandList.IsValid() ? CommandList->GetMappedCommandGroup(Command->GetCommandIdentifier()) : NAME_None;

		// Shrink padding if adjacent commands are in the same group
		if(CommandIndex < Commands.Num() - 1)
		{
			if(Group != NAME_None)
			{
				TSharedRef<SCommand>& NextCommand = Commands[CommandIndex + 1];
				TSharedPtr<const FUICommandList_Pinnable> NextCommandList = NextCommand->GetPinnableCommandList().Pin();
				if(NextCommandList.IsValid())
				{
					FName NextGroup = NextCommandList->GetMappedCommandGroup(NextCommand->GetCommandIdentifier());
					if(NextGroup == Group)
					{
						Padding.Right = 0.0f;
					}
				}
			}
		}
		else if(CommandIndex == Commands.Num() - 1)
		{
			Padding.Right = 0.0f;
		}

		CommandBox->AddSlot()
		.Padding(Padding)
		[
			Command
		];
	}
}

#undef LOCTEXT_NAMESPACE
