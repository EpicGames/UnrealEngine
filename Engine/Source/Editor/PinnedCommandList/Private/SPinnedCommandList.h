// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "IPinnedCommandList.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FUICommandInfo;
class FUICommandList;
class FUICommandList_Pinnable;
class ISlateStyle;
class SCommand;
class SWidget;
class SWrapBox;
struct FGeometry;
struct FPointerEvent;

/** Tracks a registered custom widget, so it can be persisted */
struct FRegisteredCustomWidget
{
	/** Identifier for this custom widget */
	FName CustomWidgetIdentifier;

	/** Widget creation delegate */
	IPinnedCommandList::FOnGenerateCustomWidget OnGenerateCustomWidget;

	/** Display name for this custom widget, used in context menus and tooltips */
	TAttribute<FText> CustomWidgetDisplayName;

	/** Padding to use for this custom widget */
	FMargin CustomWidgetPadding;

	/** Whether to show the label for this custom widget */
	bool bShowLabel;
};

/**
 * A list of commands that were 'pinned' by the user, for easy re-use
 */
class SPinnedCommandList : public IPinnedCommandList
{
public:
	/** Delegate for when commands have changed */
	DECLARE_DELEGATE( FOnCommandsChanged );

	DECLARE_DELEGATE_RetVal( TSharedPtr<SWidget>, FOnGetContextMenu );

	SLATE_BEGIN_ARGS( SPinnedCommandList ){}

		/** Called when a command is right clicked */
		SLATE_EVENT( FOnGetContextMenu, OnGetContextMenu )

		/** Delegate for when commands have changed */
		SLATE_EVENT( FOnCommandsChanged, OnCommandsChanged )

	SLATE_END_ARGS()

	SPinnedCommandList();
	~SPinnedCommandList();

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const FName& InContextName);

	/** IPinnedCommandList interface */
	virtual void BindCommandList(const TSharedRef<const FUICommandList>& InUICommandList) override;
	virtual void BindCommandList(const TSharedRef<FUICommandList_Pinnable>& InUICommandList_Pinnable) override;
	virtual void RegisterCustomWidget(IPinnedCommandList::FOnGenerateCustomWidget InOnGenerateCustomWidget, FName InCustomWidgetIdentifier, TAttribute<FText> InCustomWidgetDisplayName, FMargin InCustomWidgetPadding = FMargin(2.0f, 1.0f), bool bShowLabel = true) override;
	virtual void AddCommand(const TSharedRef<const FUICommandInfo>& InCommandInfo, const TSharedRef<const FUICommandList>& InUICommandList) override;
	virtual void AddCustomWidget(FName InCustomWidgetIdentifier) override;
	virtual void RemoveCommand(const TSharedRef<const FUICommandInfo>& InCommandInfo) override;
	virtual void RemoveCustomWidget(FName InCustomWidgetIdentifier) override;
	virtual bool HasAnyCommands() const override;
	virtual void RemoveAllCommands() override;
	virtual void SetStyle(const ISlateStyle* InStyleSet, const FName& InStyleName) override;

	/** Saves any settings to config that should be persistent between editor sessions */
	void SaveSettings() const;

	/** Loads any settings to config that should be persistent between editor sessions */
	void LoadSettings();

	/** SWidget interface */
	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent );

private:

	/** Adds a command and returns its widget. Creates a new widget if one does not already exist */
	TSharedRef<SCommand> AddCommand_Internal(const TSharedRef<const FUICommandInfo>& InCommandInfo, const TSharedRef<const FUICommandList>& InUICommandList, const TSharedPtr<const FUICommandList_Pinnable>& InUICommandListPinnable = nullptr);

	/** Adds a custom widget and returns its widget. Creates a new widget if one does not already exist */
	TSharedPtr<SCommand> AddCustomWidget_Internal(FName InCustomWidgetIdentifier, const TSharedPtr<const FUICommandList_Pinnable>& InUICommandListPinnable = nullptr);

	/** Adds a command to the end of the pinned commands box. */
	void AddCommandWidget(const TSharedRef<SCommand>& CommandToAdd);

	/** Removes a command from the pinned commands box. */
	void RemoveCommandWidget(const TSharedRef<SCommand>& CommandToRemove);

	/** Called when reset commands option is pressed */
	void OnResetCommands();

	/** Remove all commands but the one specified */
	void RemoveAllCommandsButThis(const TSharedRef<SCommand>& CommandToRemove);

	/** Filter action execution */
	void HandleExecuteAction(const TSharedRef<const FUICommandInfo>& InCommandInfo, const TSharedRef<const FUICommandList_Pinnable>& InUICommandList);

	/** Filter custom widget interactions */
	void HandleCustomWidgetInteraction(FName InCustomWidgetIdentifier, const TSharedRef<const FUICommandList_Pinnable>& InUICommandList);

	/** Sort commands */
	void SortCommands();

	/** Clear and re-add all widgets to slots */
	void RefreshCommandWidgets();

private:
	/** Context name used to persist this widget's settings */
	FName ContextName;

	/** The horizontal box which contains all the commands */
	TSharedPtr<SWrapBox> CommandBox;

	/** All commands in the list */
	TArray<TSharedRef<SCommand>> Commands;

	/** UI command lists that we can execute commands to (used to re-bind the UI on reload) */
	TArray<TWeakPtr<const FUICommandList>> BoundCommandLists;

	/** UI command lists that we can execute commands to (used to re-bind the UI on reload) */
	TArray<TWeakPtr<const FUICommandList_Pinnable>> BoundPinnableCommandLists;

	/** Registered custom widgets */
	TArray<FRegisteredCustomWidget> CustomWidgets;

	/** Delegate for getting the context menu. */
	FOnGetContextMenu OnGetContextMenu;

	/** Delegate for when commands have changed */
	FOnCommandsChanged OnCommandsChanged;

	/** The slate style to use when constructing command widgets */
	const ISlateStyle* StyleSet;
	
	/** The menu style name to use when constructing command widgets */
	FName StyleName;
};
