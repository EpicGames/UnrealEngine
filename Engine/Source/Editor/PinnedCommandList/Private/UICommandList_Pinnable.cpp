// Copyright Epic Games, Inc. All Rights Reserved.

#include "UICommandList_Pinnable.h"

#include "CoreTypes.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Misc/AssertionMacros.h"
#include "UObject/UnrealNames.h"

struct FUIAction;

FUICommandList_Pinnable::FUICommandList_Pinnable()
	: CurrentActionIndex(0)
{
}

bool FUICommandList_Pinnable::ExecuteAction(const TSharedRef< const FUICommandInfo > InUICommandInfo) const
{
	bool bResult = FUICommandList::ExecuteAction(InUICommandInfo);

	OnExecuteActionDelegate.Broadcast(InUICommandInfo, SharedThis(this));

	return bResult;
}

void FUICommandList_Pinnable::WidgetInteraction(FName InCustomWidgetIdentifier) const
{
	OnCustomWidgetInteractionDelegate.Broadcast(InCustomWidgetIdentifier, SharedThis(this));
}

void FUICommandList_Pinnable::MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, const FUIAction& InUIAction )
{
	FUICommandList::MapAction(InUICommandInfo, InUIAction);

	// Map the index and group
	int32 ActionIndex = InUICommandInfo->GetBindingContext().GetComparisonIndex().ToUnstableInt() + CurrentActionIndex;
	CurrentActionIndex++;

	CommandIndexMap.Add(InUICommandInfo->GetCommandName(), ActionIndex);
	if(CurrentGroupName != NAME_None)
	{
		CommandGroupMap.Add(InUICommandInfo->GetCommandName(), CurrentGroupName);
	}
}

int32 FUICommandList_Pinnable::GetMappedCommandIndex( FName InCommandName ) const
{
	const int32* FoundIndex = CommandIndexMap.Find(InCommandName);
	if(FoundIndex != nullptr)
	{
		return *FoundIndex;
	}

	return INDEX_NONE;
}

FName FUICommandList_Pinnable::GetMappedCommandGroup( FName InCommandName ) const
{
	const FName* FoundGroup = CommandGroupMap.Find(InCommandName);
	if(FoundGroup != nullptr)
	{
		return *FoundGroup;
	}

	return NAME_None;
}

void FUICommandList_Pinnable::BeginGroup(FName InGroupName)
{
	check(CurrentGroupName == NAME_None);

	CurrentGroupName = InGroupName;
}

void FUICommandList_Pinnable::EndGroup()
{
	check(CurrentGroupName != NAME_None);

	CurrentGroupName = NAME_None;
}