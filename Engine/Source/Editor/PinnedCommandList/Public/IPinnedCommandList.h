// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "IPinnedCommandListModule.h"

class FUICommandInfo;
class FUICommandList;
class FUICommandList_Pinnable;
class ISlateStyle;

/**
 * A list of commands that were 'pinned' by the user, for easy re-use
 * To use the a command list:
 * 1. Construct it using CreatePinnedCommandList (using a valid context if you want to persist settings).
 *    Optionally call SetStyle after construction to style your widget for its context. Styles are
 *    the same concatenated named styles given to menu button items.
 * 2. Bind one or more command lists to the widget via BindCommandList. This should be done *after*
 *    commands are registered to the FUICommandlist to allow commands to be correctly re-created
 *    from persistent settings.
 * 3. Register one or more custom widgets via RegisterCustomWidget.
 * 4. If using the FUICommandList_Pinnable, commands should be automatically forwarded to the pinned 
 *    commands list widget when executed. If not, then the user needs to manually add commands via 
 *    AddCommand.
 * 5. If using FUICommandList_Pinnable custom widget interactions will be automatically forwarded to
 *    the pinned commands list, but FUICommandList_Pinnable::OnCustomWidgetInteraction *does* need 
 *    to be called when widgets are interacted with to automatically add the custom widget to 
 *    the pinned commands list.
 */
class IPinnedCommandList : public SCompoundWidget
{
public:
	/** Delegate used to generate custom widgets */
	DECLARE_DELEGATE_RetVal(TSharedRef<SWidget>, FOnGenerateCustomWidget);

	/** 
	 * Bind a command list to this widget. To persist the UI between sessions, register your command lists when this widget is constructed. 
	 * @param	InUICommandList		The command list to bind
	 */
	virtual void BindCommandList(const TSharedRef<const FUICommandList>& InUICommandList) = 0;

	/** 
	 * Bind a pinnable command list to this widget. As well as calling BindCommandList(),
	 * this also sets up delegates to forward commands and widget interactions to this command list.
	 * @param	InUICommandList		The command list to bind
	 */
	virtual void BindCommandList(const TSharedRef<FUICommandList_Pinnable>& InUICommandList) = 0;

	/** 
	 * Register a custom widget. This allows the widget to be dynamically reconstructed according to the persisted commands.
	 * @param	InOnGenerateCustomWidget	Delegate used to regenerate the custom widget when created or loaded.
	 * @param	InCustomWidgetIdentifier	Unique identifier used to persist the widget. Matched to the parameter when calling AddCustomWidget.
	 * @param	InCustomWidgetDisplayName	Text to display next to the custom widget, and to generate tooltips for the widget.
	 * @param	InCustomWidgetPadding		Custom padding around the widget.
	 * @param	bInShowLabel				Whether to display the label in the pinned widget (it will still be used in the context menus)
	 */
	virtual void RegisterCustomWidget(FOnGenerateCustomWidget InOnGenerateCustomWidget, FName InCustomWidgetIdentifier, TAttribute<FText> InCustomWidgetDisplayName, FMargin InCustomWidgetPadding = FMargin(2.0f, 1.0f), bool bInShowLabel = true) = 0;

	/** 
	 * Add a command to the pinned list.
	 * @param	InUICommandInfo		The command to add
	 * @param	InUICommandList		The command list that handles the command
	 */
	virtual void AddCommand(const TSharedRef<const FUICommandInfo>& InUICommandInfo, const TSharedRef<const FUICommandList>& InUICommandList) = 0;

	/** 
	 * Add a previously registered custom widget to the pinned list. @see RegisterCustomWidget.
	 * @param	InCustomWidgetIdentifier	Unique identifier used to persist the widget.
	 */
	virtual void AddCustomWidget(FName InCustomWidgetIdentifier) = 0;

	/** 
	 * Remove a command from the pinned list.
	 * @param	InUICommandInfo		The command to remove
	 */
	virtual void RemoveCommand(const TSharedRef<const FUICommandInfo>& InUICommandInfo) = 0;

	/**
	 * Remove a custom widget from the pinned list 
	 * @param	InCustomWidgetIdentifier	Unique identifier used to persist the widget.
	 */
	virtual void RemoveCustomWidget(FName InCustomWidgetIdentifier) = 0;

	/** @return true if any commands are assigned */
	virtual bool HasAnyCommands() const = 0;

	/** Removes all commands in the list */
	virtual void RemoveAllCommands() = 0;

	/** 
	 * Set a menu style to use when building command widgets.
	 * @param	InStyleSet		The style set to use, e.g. &FAppStyle::Get()
	 * @param	InStyleName		The name of the menu button style to use, e.g. "Menu"
	 */
	virtual void SetStyle(const ISlateStyle* InStyleSet, const FName& InStyleName) = 0;
};
