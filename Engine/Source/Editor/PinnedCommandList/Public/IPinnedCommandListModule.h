// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class IPinnedCommandList;

class IPinnedCommandListModule : public IModuleInterface
{
public:
	/**
	 * Create a pinned commands list widget
	 * @param	InContextName	The context name used to persist the widget's settings
	 * @return a new pinned commands widget
	 */
	virtual TSharedRef<IPinnedCommandList> CreatePinnedCommandList(const FName& InContextName) = 0;
};
