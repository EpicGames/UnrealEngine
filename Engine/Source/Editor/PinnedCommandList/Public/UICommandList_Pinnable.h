// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "Framework/Commands/UICommandList.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API PINNEDCOMMANDLIST_API

class FUICommandInfo;
struct FUIAction;

/** 
 * Extended version of FUICommandList that allows:
 * - Hooks into action dispatch, to allow commands to be forwarded for pinning
 * - Indexing when mapping to provide a sort key
 * - Optional grouping of actions
 */
class FUICommandList_Pinnable : public FUICommandList
{
public:
	UE_API FUICommandList_Pinnable();

	using FUICommandList::MapAction;

	/** Delegate called when an action is executed */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnExecuteAction, const TSharedRef<const FUICommandInfo>& /*InUICommandInfo*/, const TSharedRef<const FUICommandList_Pinnable>& /*InUICommandList*/);

	/** Delegate called when a custom widget is interacted with */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnCustomWidgetInteraction, FName /*InCustomWidgetIdentifier*/, const TSharedRef<const FUICommandList_Pinnable>& /*InUICommandList*/);

	/** FUICommandList interface */
	UE_API virtual bool ExecuteAction(const TSharedRef< const FUICommandInfo > InUICommandInfo) const override;
	UE_API virtual void MapAction( const TSharedPtr< const FUICommandInfo > InUICommandInfo, const FUIAction& InUIAction ) override;

	/** Mark that a custom widget was interacted with */
	UE_API void WidgetInteraction(FName InCustomWidgetIdentifier) const;

	/** Delegate called when an action is executed */
	FOnExecuteAction& OnExecuteAction() { return OnExecuteActionDelegate; }

	/** Delegate called when an custom widget is interacted with */
	FOnCustomWidgetInteraction& OnCustomWidgetInteraction() { return OnCustomWidgetInteractionDelegate; }

	/** 
	 * Get the index this command was mapped to.
	 * @param	InCommandName		The command name to check
	 * @return the mapped index, or INDEX_NONE if the command is not mapped to this command list
	 */
	UE_API int32 GetMappedCommandIndex( FName InCommandName ) const;

	/** 
	 * Get the group this command was mapped into.
	 * @param	InUICommandInfo		The command to check
	 * @return the command's group, or NAME_None if the command is not grouped
	 */
	UE_API FName GetMappedCommandGroup( FName InCommandName ) const;

	/** 
	 * Start a group scope. All calls to MapAction will be grouped into the specified group.
	 * Should be paired with EndGroup()
	 */
	UE_API void BeginGroup(FName InGroupName);

	/** End a group scope. Should be paired with BeginGroup(). */
	UE_API void EndGroup();

protected:
	/** Index used to provide a sort key for actions when mapped */
	int32 CurrentActionIndex;

	/** Current group name */
	FName CurrentGroupName;

	/** Delegate called when an action is executed */
	FOnExecuteAction OnExecuteActionDelegate;

	/** Delegate called when a custom widget is interacted with */
	FOnCustomWidgetInteraction OnCustomWidgetInteractionDelegate;

	/** Map providing an index for each command that was mapped via this command list */
	TMap<FName, int32> CommandIndexMap;

	/** Map providing a group for each command that was mapped via this command list */
	TMap<FName, FName> CommandGroupMap;
};

#undef UE_API
